// Copyright Epic Games, Inc. All Rights Reserved.

#include "SpatialReadinessAPI.h"
#include "SpatialReadinessAPIDelegates.h"

FSpatialReadinessAPI::FSpatialReadinessAPI(
	FAddUnreadyVolume_Function AddUnreadyVolume,
	FRemoveUnreadyVolume_Function RemoveUnreadyVolume)
	: Delegates(MakeShared<FSpatialReadinessAPIDelegates>(AddUnreadyVolume, RemoveUnreadyVolume))
{ }

FSpatialReadinessAPI::~FSpatialReadinessAPI()
{
	if (ensureMsgf(Delegates, TEXT("An internal spatial readiness interfaec was freed before the destruction of its outer")))
	{
		// Some handle might still have reference to the internal since it has a WeakPtr
		// which can be converted to shared. That means it might try to access its
		// delegates, which will now likely be bound to invalid functions.
		Delegates->AddUnreadyVolumeDelegate.Unbind();
		Delegates->RemoveUnreadyVolumeDelegate.Unbind();
	}
}

FSpatialReadinessVolume FSpatialReadinessAPI::CreateVolume(const FBox& Bounds, const FString& Description)
{
	// Create a handle for a new unready volume
	FSpatialReadinessVolume ReadinessVolume(Delegates.ToWeakPtr(), Bounds, Description);

	// Default to an unready state. This should call back into AddUnreadyVolume_Internal
	// to create the underlying unready volume.
	ReadinessVolume.MarkUnready();

	// Return the newly created handle. This handle will only be valid so long as this
	// ISpatialReadiness object is in scope.
	return ReadinessVolume;
}
