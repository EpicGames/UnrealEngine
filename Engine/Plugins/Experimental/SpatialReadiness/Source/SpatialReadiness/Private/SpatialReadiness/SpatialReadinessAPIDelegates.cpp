// Copyright Epic Games, Inc. All Rights Reserved.

#include "SpatialReadinessAPIDelegates.h"
#include "SpatialReadinessStats.h"

FSpatialReadinessAPIDelegates::FSpatialReadinessAPIDelegates(
	FAddUnreadyVolume_Function AddUnreadyVolume,
	FRemoveUnreadyVolume_Function RemoveUnreadyVolume)
{
	// Each of these bindings captures _itself_ so it can check to see that it's still
	// bound after execution. This somewhat odd construction exists because this is the
	// only point at which we can't be sure that the interface that created the handle
	// still exists... although the only way it should be possible for it to have been
	// deleted mid-execution of one of these lambdas would be if there was some crazy
	// threading going on.
	//
	// Here's the situation that we're trying to protect against:
	// 1) Interface.CreateHandle(); // called on thread 0, triggers call to AddUnreadyVolumeDelegate.Execute(...)
	// 2) Interface.~(); // called on thread 1, before above delegate execution completes

	AddUnreadyVolumeDelegate.BindLambda(AddUnreadyVolume);
	RemoveUnreadyVolumeDelegate.BindLambda(RemoveUnreadyVolume);
}
