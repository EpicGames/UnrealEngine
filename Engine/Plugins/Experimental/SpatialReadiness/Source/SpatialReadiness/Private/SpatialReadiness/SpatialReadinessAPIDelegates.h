// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SpatialReadinessSignatures.h"

struct FSpatialReadinessAPIDelegates final
{
	// Constructor which binds TFunctions
	FSpatialReadinessAPIDelegates(
		FAddUnreadyVolume_Function AddUnreadyVolume,
		FRemoveUnreadyVolume_Function RemoveUnreadyVolume);

	~FSpatialReadinessAPIDelegates() = default;

	FAddUnreadyVolume_Delegate AddUnreadyVolumeDelegate;
	FRemoveUnreadyVolume_Delegate RemoveUnreadyVolumeDelegate;
};
