// Copyright Epic Games, Inc. All Rights Reserved.

#include "SpatialReadinessDebug.h"
#include "SpatialReadinessLog.h"
#include "SpatialReadinessSubsystem.h"
#include "SpatialReadinessSimCallback.h"
#include "PhysicsProxy/SingleParticlePhysicsProxy.h"
#include "Physics/Experimental/PhysScene_Chaos.h"
#include "PhysicsSolver.h"
#include "Chaos/ParticleHandle.h"
#include "Engine/World.h"

using Chaos::FSingleParticlePhysicsProxy;
using Chaos::FGeometryParticleHandle;
using Chaos::FAABB3;

TAutoConsoleVariable<bool> CVarSpatialReadinessDebugDraw(
	TEXT("p.SpatialReadiness.DebugDraw"),
	0,
	TEXT("Debug drawing for spatial readiness subsystem"),
	ECVF_Default);


static FAutoConsoleCommandWithWorldAndArgs CmdSpatialReadinessPrint(
	TEXT("p.SpatialReadiness.Print"),
	TEXT("Command to log all current unready volumes and frozen rigid particles"),
	FConsoleCommandWithWorldAndArgsDelegate::CreateLambda([](const TArray<FString>& Args, UWorld* World)
	{
#if !UE_BUILD_SHIPPING
		if (World == nullptr)
		{
			UE_LOGF(LogSpatialReadiness, Log, "Can't print spatial readiness: no world");
			return;
		}

		FPhysScene_Chaos* Scene = static_cast<FPhysScene_Chaos*>(World->GetPhysicsScene());
		if (Scene == nullptr)
		{
			UE_LOGF(LogSpatialReadiness, Log, "Can't print spatial readiness: no physics scene");
			return;
		}

		Chaos::FPhysicsSolver* Solver = Scene->GetSolver();
		if (Solver == nullptr)
		{
			UE_LOGF(LogSpatialReadiness, Log, "Can't print spatial readiness: no solver");
			return;
		}

		USpatialReadiness* SpatialReadiness = World->GetSubsystem<USpatialReadiness>();
		if (SpatialReadiness == nullptr)
		{
			UE_LOGF(LogSpatialReadiness, Log, "Can't print spatial readiness: no spatial readiness subsystem");
			return;
		}

		FSpatialReadinessSimCallback* SimCallback = SpatialReadiness->GetSimCallback();
		if (SimCallback == nullptr)
		{
			UE_LOGF(LogSpatialReadiness, Log, "Can't print spatial readiness: no spatial readiness sim callback");
			return;
		}

		{
			// Print unready volumes string
			int32 VolumeNum = 0;
			TStringBuilder<(1<<14)> Builder;
			Builder << "\nUnready Volumes (" << SimCallback->GetNumUnreadyVolumes_GT() << "):\n";
			SimCallback->ForEachVolumeData_GT([&Builder, &VolumeNum](const FUnreadyVolumeData_GT& Data)
			{
				FSingleParticlePhysicsProxy* Proxy = Data.Proxy;
				if (Proxy == nullptr)
				{
					return;
				}

				if (Proxy->GetMarkedDeleted())
				{
					return;
				}

				FGeometryParticleHandle* UnreadyVolume = Proxy->GetHandle_LowLevel();
				if (UnreadyVolume == nullptr)
				{
					return;
				}

				const FAABB3 Bounds = UnreadyVolume->WorldSpaceInflatedBounds();

				Builder << " " << (VolumeNum++) << ". "
#if WITH_SPATIAL_READINESS_DESCRIPTIONS
				<< Data.Description << " : "
#endif
				<< *Bounds.ToString() << "\n";
			});

			// Print the string
			UE_LOGF(LogSpatialReadiness, Log, "%ls", *Builder);
		}

		{
			// Print frozen particles string

			// Check to see if we're required to spatially sort frozen bodies
			bool bSortFrozen = false;
			for (const FString& Str : Args)
			{
				if (Str.Compare("-s", ESearchCase::IgnoreCase) == 0)
				{
					bSortFrozen = true;
					break;
				}
			}

			// Enqueue a command to print all particles
			Solver->EnqueueCommandImmediate([SimCallback, bSortFrozen]()
			{
				TStringBuilder<(1<<14)> Builder;
				Builder << "\nFrozen Particles (" << SimCallback->GetNumUnreadyRigidParticles_PT() << "):\n";

				// Make a lambda that will accumulate a string for printing frozen particles
				const auto PrintParticle = [&Builder](const Chaos::FPBDRigidParticleHandle* RigidParticle) -> bool
				{
					static int32 ParticleNum = 0;
					const FAABB3 Bounds = RigidParticle->WorldSpaceInflatedBounds();
					const FVector Pos = RigidParticle->GetX();
					const FName DebugName = *RigidParticle->GetDebugName();
					Builder << " " << (ParticleNum++) << ". " << *DebugName.ToString() << " : " << *Pos.ToString() << "\n";
					return true;
				};

				if (bSortFrozen)
				{
					TArray<const Chaos::FPBDRigidParticleHandle*> Particles;
					Particles.Reserve(SimCallback->GetNumUnreadyRigidParticles_PT());
					SimCallback->ForEachUnreadyRigidParticle_PT([&Particles](const Chaos::FPBDRigidParticleHandle* RigidParticle) -> bool
					{
						Particles.Add(RigidParticle);
						return true;
					});
					Particles.Sort([](const Chaos::FPBDRigidParticleHandle& A, const Chaos::FPBDRigidParticleHandle& B) -> bool
					{
						const FVector AX = A.GetX();
						const FVector BX = B.GetX();
						return
							(AX.X < BX.X) || (
								(AX.X == BX.X) && (
									(AX.Y < BX.Y) || (
										(AX.Y == BX.Y) && (AX.Z < BX.Z)
									)
								)
							);
					});
					for (const Chaos::FPBDRigidParticleHandle* Particle : Particles)
					{
						PrintParticle(Particle);
					}
				}
				else
				{
					SimCallback->ForEachUnreadyRigidParticle_PT(PrintParticle);
				}

				// Print the string
				UE_LOGF(LogSpatialReadiness, Log, "%ls", *Builder);
			});
		}
#endif
	}));
