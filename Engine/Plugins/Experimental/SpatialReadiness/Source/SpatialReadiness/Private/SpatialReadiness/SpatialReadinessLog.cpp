// Copyright Epic Games, Inc. All Rights Reserved.
#include "SpatialReadinessLog.h"

DEFINE_LOG_CATEGORY(LogSpatialReadiness);