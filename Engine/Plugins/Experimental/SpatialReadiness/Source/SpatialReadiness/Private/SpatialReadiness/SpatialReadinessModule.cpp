// Copyright Epic Games, Inc. All Rights Reserved.

#include "SpatialReadinessModule.h"
#include "Modules/ModuleManager.h"

IMPLEMENT_MODULE(FDefaultModuleImpl, SpatialReadiness)

void FSpatialReadinessModule::StartupModule() { }

void FSpatialReadinessModule::ShutdownModule() { }