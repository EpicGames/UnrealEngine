// Copyright Epic Games, Inc. All Rights Reserved.

#include "SpatialReadinessSimCallback.h"
#include "Chaos/Box.h"
#include "Chaos/ChaosEngineInterface.h"
#include "Chaos/Collision/SimSweep.h"
#include "Chaos/MidPhaseModification.h"
#include "Chaos/ParticleHandle.h"
#include "Chaos/PBDRigidsEvolutionGBF.h"
#include "SpatialReadinessVolume.h"
#include "PBDRigidsSolver.h"
#include "PhysicsProxy/SingleParticlePhysicsProxy.h"
#include "Physics/PhysicsFiltering.h"
#include "Physics/PhysicsInterfaceUtils.h"
#include "Physics/Experimental/ChaosInterfaceWrapper.h"
#include "Physics/Experimental/PhysScene_Chaos.h"
#include "Physics/GenericPhysicsInterface.h"
#include "SpatialReadinessStats.h"
#include "Engine/OverlapResult.h"
#include "PhysicsEngine/PhysicsObjectExternalInterface.h"
#include "DrawDebugHelpers.h"
#include "Chaos/ChaosDebugDraw.h"
#include "Chaos/DebugDrawQueue.h"
#include "SpatialReadinessLog.h"
#include "SpatialReadinessDebug.h"
#include "SpatialReadinessDefines.h"

using namespace Chaos;

namespace
{
	bool IsParticleValid(Chaos::FGeometryParticleHandle* ParticleHandle, TSharedPtr<FProxyTimestampBase, ESPMode::ThreadSafe> SyncTimestamp)
	{
		if (ParticleHandle == nullptr)
		{
			return false;
		}

		if (SyncTimestamp.IsValid() == false)
		{
			return false;
		}

		if (SyncTimestamp->bDeleted)
		{
			return false;
		}

		IPhysicsProxyBase* Proxy = ParticleHandle->PhysicsProxy();
		if (Proxy == nullptr)
		{
			return false;
		}

		if (!ensureMsgf(Proxy->GetSyncTimestamp() == SyncTimestamp, TEXT("particle's sync timestamp doesn't match the sync timestamp that was passed to the particle validity check!")))
		{
			return false;
		}

		return true;
	};
}

TAutoConsoleVariable<int32> CVarSpatialReadiness_SimTickFreezeFrames(
	TEXT("p.SpatialReadiness.SimTickFreezeFrames"), 2,
	TEXT("When a rigid particle is frozen by contact with an unready volume, this is the number of frames which it will remain frozen after losing contact. Theoretically it could be zero, but chaos can take 1-2 frames to fully initialize the collision of new particles, and so we default it to 2 to be safe. Please avoid the temptation to use this as an indefinitely extendable \"freeze timer\" to cover up more serious timing issues in systems which use readiness."),
	ECVF_Default);

TAutoConsoleVariable<int32> CVarSpatialReadiness_MidPhaseStrategy(
	TEXT("p.SpatialReadiness.MidPhaseStrategy"), 3,
	TEXT("0: Loop over unready volumes first, midphases second. 1: Loop over all midphases, unready volumes second. 2: (NOT IMPLEMENTED YET) Use a grid acceleration structure for a more optimal version of the unready-volumes-first approach. 3: Dynamically choose between strategies based on how many midphases there are and vs unready volumes."),
	ECVF_Default);

TAutoConsoleVariable<float> CVarSpatialReadiness_MidPhaseStrategy_Dynamic_ThresholdRatioMidPhasesToUnreadyVolumes(
	TEXT("p.SpatialReadiness.MidPhaseStrategy.Dynamic.ThresholdRatioMidPhasesToUnreadyVolumes"), 1,
	TEXT("The ratio of mid phases to unready volumes below which we prioritize mid phases and above which we prioritize unready volumes."),
	ECVF_Default);

TAutoConsoleVariable<bool> CVarSpatialReadiness_CreateQueryShape(
	TEXT("p.SpatialReadiness.CreateQueryShape"), false,
	TEXT("When true, spatial readiness will create a sim and a query shape. When false, the query shape is skipped."),
	ECVF_Default);

// Callback returns Overlap for all sim shapes, None for everything else.
struct FSpatialReadinessCollisionQueryFilterCallback : public FOverlapAllCollisionQueryFilterCallback
{
	virtual ~FSpatialReadinessCollisionQueryFilterCallback() = default;

	ECollisionQueryHitType PreFilter(const Chaos::FPerShapeData& Shape)
	{
		if (!Shape.GetSimEnabled())
		{
			return ECollisionQueryHitType::None;
		}
		return ECollisionQueryHitType::Touch;
	}

	virtual ECollisionQueryHitType PreFilter(const FQueryFilterData& FilterData, const Chaos::FPerShapeData& Shape, const Chaos::FGeometryParticle& Actor) override
	{
		return PreFilter(Shape);
	}
	virtual ECollisionQueryHitType PreFilter(const FQueryFilterData& FilterData, const Chaos::FPerShapeData& Shape, const Chaos::FGeometryParticleHandle& Actor) override
	{
		return PreFilter(Shape);
	}
};

FSpatialReadinessSimCallback::FSpatialReadinessSimCallback(FPhysScene_Chaos& InPhysicsScene)
	: PhysicsScene(InPhysicsScene)
	, UnreadyVolumeData_GT(256)
	, ParticleDataCache_PT()
{ }

FPBDRigidsEvolution* FSpatialReadinessSimCallback::GetEvolution()
{
	if (FPBDRigidsSolver* MySolver = static_cast<FPBDRigidsSolver*>(GetSolver()))
	{
		return MySolver->GetEvolution();
	}

	return nullptr;
}

int32 FSpatialReadinessSimCallback::AddUnreadyVolume_GT(const FBox& Bounds, const FString& Description)
{
	SCOPE_CYCLE_COUNTER(STAT_SpatialReadiness_Physics_AddUnreadyVolumeGT)

	// Get a pointer to the input struct - if this is null just cancel. If we create a
	// particle without the input, then we'll lose track of it.
	FSpatialReadinessSimCallbackInput* SimInputPtr = GetProducerInputData_External();
	if (!ensureMsgf(SimInputPtr, TEXT("Failed to access sim callback object input")))
	{
		return INDEX_NONE;
	}
	FSpatialReadinessSimCallbackInput& SimInput = *SimInputPtr;

	// Create a box implicit geometry from the bounds
	const FVec3 BoxCenter = (Bounds.Min + Bounds.Max) * .5f;
	const FVec3 BoxHalfExtent = (Bounds.Max - Bounds.Min) * .5f;
	Chaos::FImplicitObjectPtr BoxGeom = MakeImplicitObjectPtr<TBox<FReal, 3>>(-BoxHalfExtent, BoxHalfExtent);

	// Create a new static particle to represent the volume
	FActorCreationParams Params;
	Params.bSimulatePhysics = false;
	Params.bStatic = true;
	Params.InitialTM = FTransform(FQuat::Identity, BoxCenter);
	Params.Scene = &PhysicsScene;
	FSingleParticlePhysicsProxy* ParticleProxy = nullptr;
	FChaosEngineInterface::CreateActor(Params, ParticleProxy);
	if (!ensureMsgf(ParticleProxy, TEXT("Failed to create new particle proxy")))
	{
		return INDEX_NONE;
	}
	FRigidBodyHandle_External& ParticleHandle = ParticleProxy->GetGameThreadAPI();
	
	// Create collision filter data for the particle
	// Notes of query filter data:
	// * We want to block all channels in sim but overlap all channels in query.
	Chaos::Filter::FShapeFilterBuilder FilterBuilder;
	FilterBuilder.SetCollisionChannelIndex(ECollisionChannel::ECC_WorldDynamic);
	FilterBuilder.SetOverlapChannelMask(TNumericLimits<uint64>::Max());
	FilterBuilder.SetBlockChannelMask(TNumericLimits<uint64>::Max());
	FilterBuilder.SetFilterFlags(Chaos::EFilterFlags::SimpleCollision);
	FilterBuilder.SetFilterFlags(Chaos::EFilterFlags::ComplexCollision);
	FilterBuilder.SetFilterFlags(Chaos::EFilterFlags::CCD);
	FilterBuilder.SetFilterFlags(Chaos::EFilterFlags::StaticShape);
	const Chaos::Filter::FShapeFilterData SimShapeFilterData = FilterBuilder.Build();

	// Make the geometry
	const bool bCreateQueryShape = CVarSpatialReadiness_CreateQueryShape.GetValueOnAnyThread();

	Chaos::FImplicitObjectPtr GeomPtr = BoxGeom;
	if (bCreateQueryShape)
	{
		TArray<Chaos::FImplicitObjectPtr> Geometry{ BoxGeom, BoxGeom };
		GeomPtr = MakeImplicitObjectPtr<Chaos::FImplicitObjectUnion>(MoveTemp(Geometry));
	}
	ParticleHandle.SetGeometry(GeomPtr);

	ParticleHandle.SetShapeSimCollisionEnabled(0, true);
	ParticleHandle.SetShapeQueryCollisionEnabled(0, false);
	ParticleHandle.SetShapeFilterData(0, SimShapeFilterData);
	ParticleHandle.ShapesArray()[0]->SetIsProbe(true);

	if (bCreateQueryShape)
	{
		// The query filter is the same as sim, just with all blocking disabled.
		Chaos::Filter::FShapeFilterBuilder QueryFilterBuilder(SimShapeFilterData);
		QueryFilterBuilder.SetBlockChannelMask(0);
		const Chaos::Filter::FShapeFilterData QueryShapeFilterData = QueryFilterBuilder.Build();

		ParticleHandle.SetShapeSimCollisionEnabled(1, false);
		ParticleHandle.SetShapeQueryCollisionEnabled(1, true);
		ParticleHandle.SetShapeFilterData(1, QueryShapeFilterData);
		ParticleHandle.ShapesArray()[1]->SetIsProbe(false);
	}
	
#if CHAOS_DEBUG_NAME
	ParticleHandle.SetDebugName(MakeShared<FString>(FString::Printf(TEXT("UnreadyVolume: %s"), *Description)));
#endif

	// Add the new particle to the scene
	{
		FScopedSceneLock_Chaos SceneLock(&PhysicsScene, EPhysicsInterfaceScopedLockType::Write);
		TArray<FPhysicsActorHandle> Actors = { ParticleProxy };
		PhysicsScene.AddActorsToScene_AssumesLocked(Actors);
	}

	// Save the proxy in our list of GT particles.
	//
	// Technically, the index of the volume just has to be unique, it doesn't
	// have to be the same as the particle index. However, we do this
	// just so that when querying there's no need to map from hit particles
	// back to volume index - we'll already have direct access to the index.
	const int32 ParticleIndex = ParticleHandle.UniqueIdx().Idx;
	const int32 VolumeIndex = ParticleIndex;
	const bool bAdded = UnreadyVolumeData_GT.TryAdd(VolumeIndex,
		FUnreadyVolumeData_GT(
			ParticleProxy,
			Bounds,
			Description));
	ensureMsgf(bAdded, TEXT("Failed to add volume data to map - VolumeIndex already exists!"));

	// Queue up the particle proxy for processing on PT
	SimInput.UnreadyVolumesToAdd.Add(ParticleProxy);
	SimInput.UnreadyVolumesToRemove.Remove(ParticleProxy);

	// Log the event
	UE_LOGF(LogSpatialReadiness, Verbose, "Unready volume created.with index [%d]: %ls", VolumeIndex, *Description);

#if WITH_SPATIAL_READINESS_CVD
	{
		// Draw a box in CVD representing the unready volume
		SCOPE_CYCLE_COUNTER(STAT_SpatialReadiness_Physics_ChaosVisualDebugger)
		const FName Tag = *FString::Printf(TEXT("Created: %s"), *Description);
		const FLinearColor Color = FLinearColor::Green;
		CVD_TRACE_DEBUG_DRAW_BOX(FBox(Bounds.Min, Bounds.Max), Tag, Color.ToFColorSRGB(), FChaosVisualDebuggerTrace::GetSolverID(*GetSolver()));
	}
#endif

	return VolumeIndex;
}

void FSpatialReadinessSimCallback::RemoveUnreadyVolume_GT(int32 UnreadyVolumeIndex)
{
	SCOPE_CYCLE_COUNTER(STAT_SpatialReadiness_Physics_RemoveUnreadyVolumeGT)

	FUnreadyVolumeData_GT* VolumeData = UnreadyVolumeData_GT.Find(UnreadyVolumeIndex);
	if (!ensureMsgf(VolumeData, TEXT("Trying to remove unready volume whos index is not being tracked")))
	{
		return;
	}

#if WITH_SPATIAL_READINESS_CVD
	{
		SCOPE_CYCLE_COUNTER(STAT_SpatialReadiness_Physics_ChaosVisualDebugger)
#if WITH_SPATIAL_READINESS_DESCRIPTIONS
		const FName Tag = *FString::Printf(TEXT("Destroyed: %s"), *VolumeData->Description);
#else
		const FName Tag = TEXT("Destroyed: Unready Volume");
#endif
		const FLinearColor Color = FLinearColor::Red;
		CVD_TRACE_DEBUG_DRAW_BOX(VolumeData->Bounds, Tag, Color.ToFColorSRGB(), FChaosVisualDebuggerTrace::GetSolverID(*GetSolver()));
	}
#endif

	// Get the proxy associated with this index
	FSingleParticlePhysicsProxy* ParticleProxy = VolumeData->Proxy;
	if (!ensureMsgf(ParticleProxy, TEXT("Particle proxy associated with unready volume index was null")))
	{
		return;
	}

	if (ensureMsgf(UnreadyVolumeData_GT.Find(UnreadyVolumeIndex), TEXT("No volume data associated with unready volume index")))
	{
		// Free the index in our GT tracker
		UnreadyVolumeData_GT.Remove(UnreadyVolumeIndex);
	}
	else
	{
		UE_LOGF(LogSpatialReadiness, Warning, "Attempted to remove unready volume with index %d, but it wasn't found!", UnreadyVolumeIndex);
	}

	// Tell the PT to remove it's tracking of this proxy as well
	FSpatialReadinessSimCallbackInput* SimInput = GetProducerInputData_External();
	if (ensureMsgf(SimInput, TEXT("Failed to access sim input data")))
	{
		SimInput->UnreadyVolumesToRemove.Add(ParticleProxy);
		SimInput->UnreadyVolumesToAdd.Remove(ParticleProxy);
	}

	// Delete the particle
	FChaosEngineInterface::ReleaseActor(ParticleProxy, &PhysicsScene);

	// Log the event
	UE_LOGF(LogSpatialReadiness, Verbose, "Unready volume removed with index [%d]", UnreadyVolumeIndex);
}

bool FSpatialReadinessSimCallback::QueryReadiness_GT(const FBox& Bounds, TArray<int32>& OutVolumeIndices, bool bAllUnreadyVolumes) const
{
	// Constants that we'll use to set up query parameters
	constexpr static ECollisionChannel Channel = ECollisionChannel::ECC_PhysicsBody;//Static;
	constexpr static uint8 ChannelBit = static_cast<uint8>(Channel);
	constexpr static bool bComplex = false;
	const bool bMulti = bAllUnreadyVolumes;

	// Query objects
	FCollisionQueryParams QueryParams(SCENE_QUERY_STAT(ReadinessQuery), false);
	QueryParams.bTraceComplex = false;
	FCollisionObjectQueryParams ObjectParams(static_cast<uint8>(Channel));
	FCollisionResponseContainer ResponseContainer;
	const Chaos::Filter::FQueryFilterData CollisionFilterData = CreateChaosQueryFilterData(ChannelBit, bComplex, ResponseContainer, QueryParams, ObjectParams, bMulti);
	FSpatialReadinessCollisionQueryFilterCallback QueryCallback;
	EQueryFlags QueryFlags = EQueryFlags::PreFilter;// | EQueryFlags::SkipNarrowPhase;

	// Create the geometry needed for the query
	Chaos::FImplicitBox3 Geom = Chaos::FImplicitBox3(Bounds.Min, Bounds.Max, 0);

	// Do the query
	FPhysicsHitCallback<ChaosInterface::FOverlapHit> HitBuffer;
	FPhysicsCommand::ExecuteRead(&PhysicsScene,
	[this, &Geom, &HitBuffer, &QueryFlags, &CollisionFilterData, &QueryCallback]()
	{
		const ChaosInterface::FSceneQueryCommonParams CommonParams(QueryCallback, CollisionFilterData, QueryFlags);
		Chaos::Private::LowLevelOverlap(PhysicsScene, Geom, FTransform::Identity, HitBuffer, CommonParams);
	});

	// Make sure the hits from the buffer are actually in our list of unready volumes,
	// and collect their indices
	OutVolumeIndices.Reset();
	for (int32 HitIdx = 0; HitIdx < HitBuffer.GetNumHits(); ++HitIdx)
	{
		// Get the hit result from this index
		const ChaosInterface::FOverlapHit& Hit = HitBuffer.GetHits()[HitIdx];

		// Get the particle that we hit
		Chaos::FGeometryParticle* HitParticle = Hit.Actor;
		if (HitParticle == nullptr)
		{
			continue;
		}

		// Get the particle index
		//
		// NOTE: Since we're just directly using particle unique indices for
		// volume indices, this mapping is simplified. We may at some point
		// want to use a more complex mapping though, in which case we'll need
		// to do something different here.
		const uint32 VolumeIndex = HitParticle->UniqueIdx().Idx;

		// Make sure that we have an actual entry for this particle in our
		// unready volumes list
		//if (!UnreadyVolumeData_GT.IsValidIndex(VolumeIndex))
		if (UnreadyVolumeData_GT.Find(VolumeIndex) == nullptr)
		{
			continue;
		}

		// Add the index to the output list
		OutVolumeIndices.Add(VolumeIndex);
	}

	// If we didn't hit any unready volumes then that means this volume is "ready"
	return OutVolumeIndices.Num() == 0;
}

bool FSpatialReadinessSimCallback::IsUnreadyVolume_PT(const FGeometryParticleHandle& ParticleHandle) const
{
	// Start with the cheapest checks... eliminate particles which cannot be unready volumes
	// based on state
	if (ParticleHandle.ObjectState() != EObjectStateType::Static)
	{
		return false;
	}

	// The most expensive check - see if the particle is in our set of unready volumes
	if (!UnreadyVolumeParticles_PT.Contains(static_cast<const FSingleParticlePhysicsProxy*>(ParticleHandle.PhysicsProxy())))
	{
		return false;
	}

	return true;
}

const FUnreadyVolumeData_GT* FSpatialReadinessSimCallback::GetVolumeData_GT(int32 VolumeIndex) const
{
	return UnreadyVolumeData_GT.Find(VolumeIndex);
}

void FSpatialReadinessSimCallback::ForEachVolumeData_GT(const TFunction<void(const FUnreadyVolumeData_GT&)>& Func)
{
	for (int32 Index = 0; Index < UnreadyVolumeData_GT.Num(); ++Index)
	{
		Func(UnreadyVolumeData_GT.At(Index));
	}
}

int32 FSpatialReadinessSimCallback::GetNumUnreadyVolumes_GT() const
{
	return UnreadyVolumeData_GT.Num();
}

bool FSpatialReadinessSimCallback::QueryReadiness_PT(const FAABB3& Bounds, TArray<const FSingleParticlePhysicsProxy*>& OutVolumeProxies)
{
	// Get the evolution, and get the acceleration structure from it
	FPBDRigidsEvolution* Evolution = GetEvolution();
	if (Evolution == nullptr)
	{
		return false;
	}

	// Set up particle filters for the query, so that we only get unready volumes
	// NOTE: If we could speed up this bit here, that'd be great
	const auto& ParticleFilter = [this](const FGeometryParticleHandle* Particle) -> bool
	{
		// If the particle has a single particle physics proxy (and it does,
		// I can almost guarantee it), then count it as a hit if it's in our
		// list of unready volume particles.
		if (const IPhysicsProxyBase* Proxy = Particle->PhysicsProxy())
		{
			if (Proxy->GetType() == EPhysicsProxyType::SingleParticleProxy)
			{
				return UnreadyVolumeParticles_PT.Contains(
					static_cast<const Chaos::FSingleParticlePhysicsProxy*>(Proxy));
			}
		}
		return false;
	};

	// We know that unready volumes will all be boxes, so in theory we could
	// probably filter a bunch of interactions before getting to the particle
	// filter, however the particle filter is applied first so there's no point
	// in implementing a shape filter.
	const auto& ShapeFilter = [](const FPerShapeData* Shape, const FImplicitObject* Implicit)
	{
		return true;
	};

	// Make a lambda for collecting 
	TArray<Private::FSimOverlapParticleShape> Overlaps;
	const auto& OverlapCollector = [&Overlaps](const Private::FSimOverlapParticleShape& Overlap)
	{
		Overlaps.Add(Overlap);
	};

	// Get the broadphase and query against it to see if this new rigid particle
	// generates midphases with any unready volumes.
	//
	// NOTE: In theory we could have a first-hit version of this function which
	// would potentially avoid some unnecessary checking.
	//
	// NOTE: The above note is only valid until we actually start registering
	// freeze locks with unready volumes...
	Private::SimOverlapBounds(Evolution->GetSpatialAcceleration(), Bounds, ParticleFilter, ShapeFilter, OverlapCollector);

	// Convert overlaps to list of hit proxies
	OutVolumeProxies.Reset();
	for (const Private::FSimOverlapParticleShape& Overlap : Overlaps)
	{
		const Chaos::FGeometryParticleHandle* ParticleHandle = Overlap.HitParticle;
		if (ParticleHandle == nullptr)
		{
			continue;
		}

		const IPhysicsProxyBase* Proxy = ParticleHandle->PhysicsProxy();
		if (Proxy == nullptr)
		{
			continue;
		}

		if (Proxy->GetType() != EPhysicsProxyType::SingleParticleProxy)
		{
			continue;
		}

		OutVolumeProxies.Add(static_cast<const Chaos::FSingleParticlePhysicsProxy*>(Proxy));
	}

	// Return true if there were NO overlaps - meaning, the volume is ready
	return OutVolumeProxies.Num() == 0;
}

void FSpatialReadinessSimCallback::FreezeParticles_PT()
{
	SCOPE_CYCLE_COUNTER(STAT_SpatialReadiness_Physics_FreezeParticlesPT)

	FPBDRigidsEvolution* Evolution = GetEvolution();
	if (!ensureMsgf(Evolution, TEXT("Attempted to freeze particle, but had no evolution")))
	{
		return;
	}

	ForEachUnreadyRigidParticle_PT([&](Chaos::FPBDRigidParticleHandle* RigidParticle, const int FreezeFrames)
	{
		const EObjectStateType ObjectState = RigidParticle->ObjectState();
		if (ObjectState == EObjectStateType::Static)
		{
			// This object is already static, so continue on to the next one
			return true;
		}

		// Get the geometry particle
		FGeometryParticleHandle* GeometryParticle = RigidParticle;

		// Cache the current object state
		ParticleDataCache_PT.Add({ GeometryParticle, ObjectState });

		// Set the object state to static
		Evolution->SetParticleObjectState(RigidParticle, EObjectStateType::Static);

#if ENABLE_DRAW_DEBUG
		if (CVarSpatialReadinessDebugDraw.GetValueOnAnyThread())
		{
			// Draw debug box around the frozen object
			SCOPE_CYCLE_COUNTER(STAT_SpatialReadiness_Physics_DebugDraw)
			const FAABB3 Bounds = RigidParticle->WorldSpaceInflatedBounds();
			Chaos::FDebugDrawQueue::GetInstance().DrawDebugBox(Bounds.Center(), Bounds.Extents() * .5f, FQuat::Identity, FColor::Yellow, false, -1.f, -1, 0.f);
		}
#endif

#if WITH_SPATIAL_READINESS_CVD
		{
			// Draw a box in CVD representing the frozen object
			SCOPE_CYCLE_COUNTER(STAT_SpatialReadiness_Physics_ChaosVisualDebugger)
			const FName Tag = *FString::Printf(TEXT("%s (%d)"), *RigidParticle->GetDebugName().Left(25), FreezeFrames);
			const FAABB3 Bounds = RigidParticle->WorldSpaceInflatedBounds();
			CVD_TRACE_DEBUG_DRAW_BOX(FBox(Bounds.Min(), Bounds.Max()), Tag, FColor::Yellow, FChaosVisualDebuggerTrace::GetSolverID(*GetSolver()));
		}
#endif

		// Continue on to the next one
		return true;
	});
}

void FSpatialReadinessSimCallback::UnFreezeParticles_PT()
{
	SCOPE_CYCLE_COUNTER(STAT_SpatialReadiness_Physics_UnFreezeParticlesPT)

	FPBDRigidsEvolution* Evolution = GetEvolution();
	if (!ensureMsgf(Evolution, TEXT("Attempted to un-freeze particles, but had no evolution")))
	{
		return;
	}

	// For each particle that we froze, unfreeze it
	for (TPair<FGeometryParticleHandle*, EObjectStateType>& ParticleData : ParticleDataCache_PT)
	{
		FPBDRigidParticleHandle* RigidParticle = ParticleData.Key->CastToRigidParticle();
		if (RigidParticle == nullptr)
		{
			continue;
		}

		// Restore the object state of this particle
		Evolution->SetParticleObjectState(RigidParticle, ParticleData.Value);
	}

	// Clear the array
	ParticleDataCache_PT.Reset();
}

void FSpatialReadinessSimCallback::DecrementUnreadyRigidParticles_PT()
{
	SCOPE_CYCLE_COUNTER(STAT_SpatialReadiness_Physics_DecrementUnreadyRigidParticlesPT)

	// Decrement each counter, 
	for (auto It = UnreadyRigidParticles_PT.CreateIterator(); It; ++It)
	{
		if (IsParticleValid(It->Key, It->Value.SyncTimeStamp.Pin()) == false || --It->Value.FreezeFrames < 0)
		{
			It.RemoveCurrent();
		}
	}
}

void FSpatialReadinessSimCallback::OnPreSimulate_Internal()
{
	SCOPE_CYCLE_COUNTER(STAT_SpatialReadiness_Physics_PreSimulate)

	// Process inputs we may have gotten from the game thread
	if (const FSpatialReadinessSimCallbackInput* Input = GetConsumerInput_Internal())
	{
		// Process additions
		for (FSingleParticlePhysicsProxy* ParticleProxy : Input->UnreadyVolumesToAdd)
		{
			UnreadyVolumeParticles_PT.Add(ParticleProxy);
		}

		// Process removals
		for (FSingleParticlePhysicsProxy* ParticleProxy : Input->UnreadyVolumesToRemove)
		{
			UnreadyVolumeParticles_PT.Remove(ParticleProxy);
		}
	}

	DecrementUnreadyRigidParticles_PT();
}

void FSpatialReadinessSimCallback::OnParticlesRegistered_Internal(TArray<FSingleParticlePhysicsProxy*>& RegisteredProxies)
{
	SCOPE_CYCLE_COUNTER(STAT_SpatialReadiness_Physics_ParticlesRegistered)

	// The number of frames we keep frozen particles frozen
	const int32 FreezeFrames = FMath::Max(1, CVarSpatialReadiness_SimTickFreezeFrames.GetValueOnAnyThread());

	// For each newly added particle, query against unready volumes to see if
	// it should be frozen.
	for (FSingleParticlePhysicsProxy* ParticleProxy : RegisteredProxies)
	{
		FGeometryParticleHandle* GeometryParticle = ParticleProxy->GetHandle_LowLevel();
		if (GeometryParticle == nullptr)
		{
			continue;
		}

		FPBDRigidParticleHandle* RigidParticle = GeometryParticle->CastToRigidParticle();
		if (RigidParticle == nullptr)
		{
			continue;
		}

		TWeakPtr<FProxyTimestampBase, ESPMode::ThreadSafe> SyncTimeStamp = ParticleProxy->GetSyncTimestamp();
		if (SyncTimeStamp.IsValid() == false)
		{
			continue;
		}

		// Get the bounds to use for the query
		TArray<const FSingleParticlePhysicsProxy*> VolumeProxies;
		const FAABB3 QueryBounds = RigidParticle->WorldSpaceInflatedBounds();
		if (!QueryReadiness_PT(QueryBounds, VolumeProxies))
		{
			// If this volume hit an unready volume, query readiness returns false
			// and we must mark the particle as frozen. We freeze for "+1" frames
			// because it actually takes 2 frames for new particles to show up in
			// midphases.
			UnreadyRigidParticles_PT.FindOrAdd(RigidParticle) = { SyncTimeStamp, FreezeFrames + 1 };
		}
	}
}

void FSpatialReadinessSimCallback::OnMidPhaseModification_Internal(FMidPhaseModifierAccessor& Accessor)
{
	SCOPE_CYCLE_COUNTER(STAT_SpatialReadiness_Physics_MidPhase)

	switch (GetMidPhaseStrategy(Accessor))
	{
		case ESpatialReadinessSimCallback_MidPhaseStrategy::PrioritizeUnreadyVolumes:
			OnMidPhaseModification_PrioritizeUnreadyVolumes(Accessor);
			break;

		case ESpatialReadinessSimCallback_MidPhaseStrategy::PrioritizeMidPhases:
			OnMidPhaseModification_PrioritizeMidPhases(Accessor);
			break;

		case ESpatialReadinessSimCallback_MidPhaseStrategy::Grid:
			OnMidPhaseModification_Grid(Accessor);
			break;
	}
}

int32 FSpatialReadinessSimCallback::GetFreezeFrames()
{
	return FMath::Max(1, CVarSpatialReadiness_SimTickFreezeFrames.GetValueOnAnyThread());
}

ESpatialReadinessSimCallback_MidPhaseStrategy FSpatialReadinessSimCallback::GetMidPhaseStrategy(const Chaos::FMidPhaseModifierAccessor& Accessor) const
{
	// TODO: For now we just manually pick a strategy using a CVar.
	//       We should instead use some heuristic in here to select the best strategy
	//       for the given world setup.
	const int32 MidPhaseStrategy = CVarSpatialReadiness_MidPhaseStrategy.GetValueOnAnyThread();
	if (MidPhaseStrategy < static_cast<int32>(ESpatialReadinessSimCallback_MidPhaseStrategy::COUNT))
	{
		return ESpatialReadinessSimCallback_MidPhaseStrategy(MidPhaseStrategy);
	}
	else if (MidPhaseStrategy == 3)
	{
		const int32 NumMidPhases = Accessor.GetNumMidPhases();
		const int32 NumUnreadyVolumes = UnreadyVolumeParticles_PT.Num();
		const float RatioThreshold = CVarSpatialReadiness_MidPhaseStrategy_Dynamic_ThresholdRatioMidPhasesToUnreadyVolumes.GetValueOnAnyThread();
		const float Ratio
			= NumUnreadyVolumes > 0
			? static_cast<float>(NumMidPhases) / static_cast<float>(NumUnreadyVolumes)
			: RatioThreshold;
		return Ratio < RatioThreshold
			? ESpatialReadinessSimCallback_MidPhaseStrategy::PrioritizeMidPhases
			: ESpatialReadinessSimCallback_MidPhaseStrategy::PrioritizeUnreadyVolumes;
	}

	// Just default back to prioritizing unready volumes if the cvar is set to something crazy
	return ESpatialReadinessSimCallback_MidPhaseStrategy::PrioritizeUnreadyVolumes;
}

void FSpatialReadinessSimCallback::OnMidPhaseModification_PrioritizeUnreadyVolumes(FMidPhaseModifierAccessor& Accessor)
{
	SCOPE_CYCLE_COUNTER(STAT_SpatialReadiness_Physics_MidPhase_PrioritizeUnreadyVolumes)

	// Go through every unready volume
	for (FSingleParticlePhysicsProxy* UnreadyProxy : UnreadyVolumeParticles_PT)
	{
		FGeometryParticleHandle* UnreadyVolume = UnreadyProxy->GetHandle_LowLevel();
		if (UnreadyVolume == nullptr)
		{
			continue;
		}
		if (UnreadyProxy->GetMarkedDeleted())
		{
			for (FMidPhaseModifier& MidPhase : Accessor.GetMidPhases(UnreadyVolume))
			{
				// Disable the midphase
				MidPhase.Disable();
			}
			continue;
		}

#if ENABLE_DRAW_DEBUG
		if (CVarSpatialReadinessDebugDraw.GetValueOnAnyThread())
		{
			SCOPE_CYCLE_COUNTER(STAT_SpatialReadiness_Physics_DebugDraw)
			const FAABB3 Bounds = UnreadyVolume->WorldSpaceInflatedBounds();
			Chaos::FDebugDrawQueue::GetInstance().DrawDebugBox(Bounds.Center(), Bounds.Extents() * .5f, FQuat::Identity, FColor::Red, false, -1.f, -1, 0.f);
		}
#endif

#if WITH_SPATIAL_READINESS_CVD
		{
			SCOPE_CYCLE_COUNTER(STAT_SpatialReadiness_Physics_ChaosVisualDebugger)
			// Draw a box in CVD representing the unready volume
			const FName Tag = *UnreadyVolume->GetDebugName();
			const FLinearColor Color = FLinearColor(1.f, .25f, .5f);
			const FAABB3 Bounds = UnreadyVolume->WorldSpaceInflatedBounds();
			CVD_TRACE_DEBUG_DRAW_BOX(FBox(Bounds.Min(), Bounds.Max()), Tag, Color.ToFColorSRGB(), FChaosVisualDebuggerTrace::GetSolverID(*GetSolver()));
		}
#endif

		{
			// Go through every mid-phase which involves this volume
			for (FMidPhaseModifier& MidPhase : Accessor.GetMidPhases(UnreadyVolume))
			{
				// Disable the midphase because we know it is an Unready volume
				MidPhase.Disable();

				// Get the particle that is not the unready volume
				FGeometryParticleHandle* GeometryParticle = MidPhase.GetOtherParticle(UnreadyVolume);
				if (GeometryParticle == nullptr)
				{
					continue;
				}

				FPBDRigidParticleHandle* RigidParticle = GeometryParticle->CastToRigidParticle();
				if (RigidParticle == nullptr)
				{
					continue;
				}

				FSingleParticlePhysicsProxy* RigidProxy = static_cast<FSingleParticlePhysicsProxy*>(RigidParticle->PhysicsProxy());
				if (RigidProxy == nullptr)
				{
					continue;
				}

				TWeakPtr<FProxyTimestampBase, ESPMode::ThreadSafe> SyncTimeStamp = RigidProxy->GetSyncTimestamp();
				if (SyncTimeStamp.IsValid() == false)
				{
					continue;
				}

#if ENABLE_DRAW_DEBUG
				if (CVarSpatialReadinessDebugDraw.GetValueOnAnyThread())
				{
					SCOPE_CYCLE_COUNTER(STAT_SpatialReadiness_Physics_DebugDraw)
					const FAABB3 Bounds = RigidParticle->WorldSpaceInflatedBounds();
					Chaos::FDebugDrawQueue::GetInstance().DrawDebugBox(Bounds.Center(), Bounds.Extents() * .5f, FQuat::Identity, FColor::Yellow, false, -1.f, -1, 0.f);
				}
#endif

				// Add the particle to the list of particles to freeze
				UnreadyRigidParticles_PT.FindOrAdd(RigidParticle) = { SyncTimeStamp, GetFreezeFrames()};
			}
		}
	}
}

void FSpatialReadinessSimCallback::OnMidPhaseModification_PrioritizeMidPhases(FMidPhaseModifierAccessor& Accessor)
{
	SCOPE_CYCLE_COUNTER(STAT_SpatialReadiness_Physics_MidPhase_PrioritizeMidPhases)

	Accessor.VisitMidPhases([this](FMidPhaseModifier& MidPhase)
	{
		FGeometryParticleHandle* UnreadyVolume;
		FGeometryParticleHandle* GeometryParticle;
		MidPhase.GetParticles(&UnreadyVolume, &GeometryParticle);

		// Make sure we really have 2 particles
		if (UnreadyVolume == nullptr || GeometryParticle == nullptr)
		{
			return;
		}

		// Make sure one of them is an unready volume particle
		if (!IsUnreadyVolume_PT(*UnreadyVolume))
		{
			Swap(UnreadyVolume, GeometryParticle);
			if (!IsUnreadyVolume_PT(*UnreadyVolume))
			{
				// Neither particle was an unready volume
				return;
			}
		}

		// Disable the midphase for unready volumes
		MidPhase.Disable();

		if (UnreadyVolume->PhysicsProxy()->GetMarkedDeleted())
		{
			return;
		}

		// Make sure the other one is a rigid - this should always pass
		// because you can't have generated a midphase without at least
		// one simulating particle.
		FPBDRigidParticleHandle* RigidParticle = GeometryParticle->CastToRigidParticle();
		if (RigidParticle == nullptr)
		{
			return;
		}

		FSingleParticlePhysicsProxy* RigidProxy = static_cast<FSingleParticlePhysicsProxy*>(RigidParticle->PhysicsProxy());
		if (RigidProxy == nullptr)
		{
			return;
		}

		TWeakPtr<FProxyTimestampBase, ESPMode::ThreadSafe> SyncTimeStamp = RigidProxy->GetSyncTimestamp();
		if (SyncTimeStamp.IsValid() == false)
		{
			return;
		}

		// Add the particle to the list of particles to freeze
		UnreadyRigidParticles_PT.FindOrAdd(RigidParticle) = { SyncTimeStamp, GetFreezeFrames()};

	});
}

void FSpatialReadinessSimCallback::OnMidPhaseModification_Grid(FMidPhaseModifierAccessor& Accessor)
{
	SCOPE_CYCLE_COUNTER(STAT_SpatialReadiness_Physics_MidPhase_Grid)

	//
	// TODO: Everything!
	// For now, just call back to prioritize Unready Volumes
	//
	OnMidPhaseModification_PrioritizeUnreadyVolumes(Accessor);
}

void FSpatialReadinessSimCallback::OnPreIntegrate_Internal()
{
	FreezeParticles_PT();
}

void FSpatialReadinessSimCallback::OnPostIntegrate_Internal()
{
	UnFreezeParticles_PT();
}

void FSpatialReadinessSimCallback::OnPreSolve_Internal()
{
	FreezeParticles_PT();
}

void FSpatialReadinessSimCallback::OnPostSolve_Internal()
{
	UnFreezeParticles_PT();
}

uint32 FSpatialReadinessSimCallback::FHashMapTraits::GetElementID(const FUnreadyVolumeData_GT& Element)
{
	if (FSingleParticlePhysicsProxy* Proxy = Element.Proxy)
	{
		return Proxy->GetGameThreadAPI().UniqueIdx().Idx;
	}
	return INDEX_NONE;
}

void FSpatialReadinessSimCallback::ForEachUnreadyRigidParticle_PT(const TFunction<bool(Chaos::FPBDRigidParticleHandle*)>& Lambda) const
{
	ForEachUnreadyRigidParticle_PT([&Lambda](Chaos::FPBDRigidParticleHandle* Particle, int) { return Lambda(Particle); });
}

void FSpatialReadinessSimCallback::ForEachUnreadyRigidParticle_PT(const TFunction<bool(Chaos::FPBDRigidParticleHandle*, int)>& Lambda) const
{
	for (auto It = UnreadyRigidParticles_PT.CreateConstIterator(); It; ++It)
	{
		if (IsParticleValid(It->Key, It->Value.SyncTimeStamp.Pin()) == false)
		{
			continue;
		}
	
		if (Lambda(It->Key, It->Value.FreezeFrames) == false)
		{
			return;
		}
	}
}

int32 FSpatialReadinessSimCallback::GetNumUnreadyRigidParticles_PT() const
{
	return UnreadyRigidParticles_PT.Num();
}

