// Copyright Epic Games, Inc. All Rights Reserved.

#include "SpatialReadinessSubsystem.h"
#include "SpatialReadinessSimCallback.h"
#include "Debug/DebugDrawService.h"
#include "ShowFlags.h"
#include "Engine/World.h"
#include "Engine/Canvas.h"
#include "Engine/Engine.h"
#include "DrawDebugHelpers.h"
#include "GameFramework/Pawn.h"
#include "GameFramework/PlayerController.h"
#include "PBDRigidsSolver.h"
#include "Physics/Experimental/PhysScene_Chaos.h"
#include "PhysicsProxy/SingleParticlePhysicsProxy.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SpatialReadinessSubsystem)

TAutoConsoleVariable<bool> CVarSpatialReadiness_OnDedicatedServer(TEXT("p.SpatialReadiness.OnDedicatedServer"), 1, TEXT("Enable spatial readiness on dedicated servers"),  ECVF_Default);
TAutoConsoleVariable<bool> CVarSpatialReadiness_OnStandalone(     TEXT("p.SpatialReadiness.OnStandalone"),      0, TEXT("Enable spatial readiness on standalone servers"), ECVF_Default);
TAutoConsoleVariable<bool> CVarSpatialReadiness_OnClient(         TEXT("p.SpatialReadiness.OnClient"),          0, TEXT("Enable spatial readiness on clients"),            ECVF_Default);
TAutoConsoleVariable<bool> CVarSpatialReadiness_Debug_NeverReady( TEXT("p.SpatialReadiness.Debug.NeverReady"),  0, TEXT("When true, unready volumes are never removed"),   ECVF_Default);

const FString SpatialReadinessShowFlagName = TEXT("SpatialReadiness");
TCustomShowFlag<> SpatialReadinessShowFlag(*SpatialReadinessShowFlagName, false, SFG_Developer, NSLOCTEXT("UnrealEd", "SpatialReadinessShowFlag", "Spatial Readiness"));

using namespace Chaos;

USpatialReadiness::USpatialReadiness()
	: Super()
	, SpatialReadiness(this, &This::AddUnreadyVolume, &This::RemoveUnreadyVolume) { }

USpatialReadiness::USpatialReadiness(FVTableHelper& Helper)
	: Super(Helper)
	, SpatialReadiness(this, &This::AddUnreadyVolume, &This::RemoveUnreadyVolume) { }

bool USpatialReadiness::StaticShouldCreateSubsystem(UObject* Outer)
{
	// TODO: Where should the setting for this exist? Is it enough
	//       to just not load the physics readiness module if we don't
	//       want to use it?
	if (UWorld* MyWorld = Outer->GetWorld())
	{
		const ENetMode NetMode = MyWorld->GetNetMode();
		if (NetMode == NM_Client && CVarSpatialReadiness_OnClient.GetValueOnAnyThread())
		{
			return true;
		}
		else if (NetMode == NM_DedicatedServer && CVarSpatialReadiness_OnDedicatedServer.GetValueOnAnyThread())
		{
			return true;
		}
		else if (NetMode == NM_Standalone && CVarSpatialReadiness_OnStandalone.GetValueOnAnyThread())
		{
			return true;
		}
	}
	return false;
}

bool USpatialReadiness::ShouldCreateSubsystem(UObject* Outer) const
{
	return USpatialReadiness::StaticShouldCreateSubsystem(Outer);
}

void USpatialReadiness::OnWorldBeginPlay(UWorld& InWorld)
{
	Super::OnWorldBeginPlay(InWorld);
	CreateSimCallback();
}

#if WITH_EDITOR
void USpatialReadiness::OnDebugDraw(UCanvas* Canvas, APlayerController* PlayerController)
{
#if ENABLE_DRAW_DEBUG
	if (Canvas == nullptr) { return; }
	if (PlayerController == nullptr) {
		PlayerController = GetWorld()->GetFirstPlayerController();
	}
	if (PlayerController == nullptr) { return; }

	// Get the bounds of the player
	APawn* PlayerPawn = PlayerController->GetPawn();
	if (PlayerPawn == nullptr) { return; }
	FVector PlayerOrigin, PlayerExtents;
	PlayerPawn->GetActorBounds(true, PlayerOrigin, PlayerExtents);
	const FBox PlayerBounds = FBox(PlayerOrigin - (.5f * PlayerExtents), PlayerOrigin + (.5f * PlayerExtents));

	// Check readiness in the player's location
	TArray<FString> Descriptions;
	const bool bIsReady = QueryReadiness(PlayerBounds, Descriptions, true);

	const FLinearColor Color = bIsReady ? FLinearColor::Green : FLinearColor::Red;

	// Draw box around the bounds of the player
	DrawDebugBox(GetWorld(), PlayerOrigin, PlayerExtents, FQuat::Identity, Color.ToFColor(false));

	// Draw a box in CVD around the player
	const FName PlayerTag = *AActor::GetDebugName(PlayerPawn);
	const FLinearColor PlayerColor = FLinearColor(0.f, .5f, 1.f);
	CVD_TRACE_DEBUG_DRAW_BOX(PlayerBounds, PlayerTag, PlayerColor.ToFColorSRGB(), FChaosVisualDebuggerTrace::GetSolverID(*GetSolver()));


	int32 Y = 300;
	int32 X = 100;
	UFont* Font = GEngine->GetSmallFont();
	float CharWidth = 0.0f;
	float CharHeight = 0.0f;
	Font->GetCharSize('W', CharWidth, CharHeight);
	const float ColumnWidth = 32 * CharWidth;
	const int32 FontHeight = Font->GetMaxCharHeight() + 2.0f;
	const FLinearColor BackgroundColor = FColor::Black;

	// Draw text representing readiness state in the area where the player is
	if (bIsReady)
	{
		Canvas->Canvas->DrawTile(X - 2, Y - 2, ColumnWidth + 2, FontHeight + 2, 0, 0, 1, 1, BackgroundColor);
		Canvas->Canvas->DrawShadowedString(X, Y, TEXT("Ready"), Font, Color);
		//Canvas->SetDrawColor(Color);
		//Canvas->DrawText(GEngine->GetMediumFont(), TEXT("Ready"), 100, 100);
		Y += FontHeight + 4;
	}
	else
	{
		Canvas->Canvas->DrawTile(X - 2, Y - 2, ColumnWidth + 2, FontHeight + 2, 0, 0, 1, 1, BackgroundColor);
		Canvas->Canvas->DrawShadowedString(X, Y, TEXT("Unready"), Font, Color);
		Y += FontHeight + 4;

		for (const FString& Description : Descriptions)
		{
			Canvas->Canvas->DrawTile(X - 2, Y - 2, ColumnWidth + 2, FontHeight + 2, 0, 0, 1, 1, BackgroundColor);
			Canvas->Canvas->DrawShadowedString(X, Y, *Description, Font, Color);
			Y += FontHeight + 4;
		}
	}
#endif // ENABLE_DRAW_DEBUG
}
#endif // WITH_EDITOR

void USpatialReadiness::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);

#if WITH_EDITOR
	OnDebugDrawDelegateHandle = UDebugDrawService::Register(*SpatialReadinessShowFlagName, FDebugDrawDelegate::CreateUObject(this, &USpatialReadiness::OnDebugDraw));
#endif
}

void USpatialReadiness::Deinitialize()
{
#if WITH_EDITOR
	UDebugDrawService::Unregister(OnDebugDrawDelegateHandle);
#endif
	DestroySimCallback();
	Super::Deinitialize();
}

FSpatialReadinessVolume USpatialReadiness::AddReadinessVolume(const FBox& Bounds, const FString& Description)
{
	// Create a readiness volume and return it's "handle".
	// This call will trigger the associated AddUnreadyVolume method
	// since volumes are unready by default.
	return SpatialReadiness.CreateVolume(Bounds, Description);
}

bool USpatialReadiness::QueryReadiness(const FBox& Bounds, TArray<FString>& OutDescriptions, const bool bAllDescriptions) const
{
	// Add the physics unready volume
	if (ensureMsgf(SimCallback, TEXT("Tried to query for readiness when no sim callback exists")))
	{
		// Query for readiness volumes in the sim callback object which tracks them
		TArray<int32> VolumeIndices;
		const bool bReady = SimCallback->QueryReadiness_GT(Bounds, VolumeIndices, bAllDescriptions);

		// If the macro is set to build with descriptions, populate the output
		// descriptions array
#if WITH_SPATIAL_READINESS_DESCRIPTIONS
		if (bReady == false)
		{
			OutDescriptions.Reset(VolumeIndices.Num());
			for (int32 VolumeIndex : VolumeIndices)
			{
				OutDescriptions.Add(SimCallback->GetVolumeData_GT(VolumeIndex)->Description);
			}
		}
#endif
		return bReady;
	}

	// Default to not-ready if there was a problem
	return false;
}

uint32 USpatialReadiness::GetNumUnreadyVolumes() const
{
	if (ensureMsgf(SimCallback, TEXT("Tried to query number of unready volumes when no sim callback exists")))
	{
		return SimCallback->GetNumUnreadyVolumes_GT();
	}
	return 0;
}

void USpatialReadiness::OnUnreadyVolumeChangedDelegate_Remove(const FDelegateHandle& Handle)
{
	OnUnreadyVolumeChanged_Delegate.Remove(Handle);
}

int32 USpatialReadiness::AddUnreadyVolume(const FBox& Bounds, const FString& Description)
{
	// Trigger external callback for this event
	if (OnUnreadyVolumeChanged_Delegate.IsBound())
	{
		OnUnreadyVolumeChanged_Delegate.Broadcast(Bounds, Description, EUnreadyVolumeAction::Added);
	}

	// Add the physics unready volume
	if (ensureMsgf(SimCallback, TEXT("Tried to add unready volume when no sim callback exists")))
	{
		return SimCallback->AddUnreadyVolume_GT(Bounds, Description);
	}

	return INDEX_NONE;
}

void USpatialReadiness::RemoveUnreadyVolume(int32 UnreadyVolumeIndex)
{
	// Disallow removal of unready volumes for debugging
	if (CVarSpatialReadiness_Debug_NeverReady.GetValueOnAnyThread())
	{
		return;
	}

	// Trigger external callback for this event
	if (OnUnreadyVolumeChanged_Delegate.IsBound())
	{
		if (SimCallback)
		{
			if (const FUnreadyVolumeData_GT* Data = SimCallback->GetVolumeData_GT(UnreadyVolumeIndex))
			{
#if WITH_SPATIAL_READINESS_DESCRIPTIONS
				const FString Description = Data->Description;
#else
				static const FString Description = FString("None");
#endif
				OnUnreadyVolumeChanged_Delegate.Broadcast(Data->Bounds, Description, EUnreadyVolumeAction::Removed);
			}
		}
	}

	// Remove the physics unready volume
	// NOTE: This is not an ensure because if we are removing a volume from a simcallback which
	// doesn't exist, then the volume must already have been destroyed.
	if (SimCallback)
	{
		SimCallback->RemoveUnreadyVolume_GT(UnreadyVolumeIndex);
	}
}

bool USpatialReadiness::CreateSimCallback()
{
	// If we already have a sim callback, destroy it.
	if (SimCallback)
	{
		DestroySimCallback();
	}

	// If we still have a sim callback at this point, then we
	// must have failed to delete it which means proceding would
	// leave a dangling pointer.
	if (!ensureMsgf(SimCallback == nullptr, TEXT("Tried and failed to destroy existing sim callback so that a new one could be created.")))
	{
		return false;
	}

	// The sim callback takes a scene reference in its constructor
	FPhysScene_Chaos* Scene = GetScene();
	if (!ensureMsgf(Scene, TEXT("Trying to create sim callback when there's no physics scene")))
	{
		return false;
	}

	// We need the solver to create the scene callback
	Chaos::FPhysicsSolver* Solver = GetSolver();
	if (!ensureMsgf(Solver, TEXT("Trying to create sim callback when there's no physics solver")))
	{
		return false;
	}

	// Request creation of the scene callback from the solver
	SimCallback = Solver->CreateAndRegisterSimCallbackObject_External<FSpatialReadinessSimCallback>(*Scene);
	if (!ensureMsgf(SimCallback, TEXT("Sim callback creation failed")))
	{
		return false;
	}

	// Return true to indicate successful creation of SimCallback
	return true;
}

bool USpatialReadiness::DestroySimCallback()
{
	if (SimCallback)
	{
		if (Chaos::FPhysicsSolver* Solver = GetSolver())
		{
			Solver->UnregisterAndFreeSimCallbackObject_External(SimCallback);
			SimCallback = nullptr;
			return true;
		}
	}

	return false;
}

FPhysScene_Chaos* USpatialReadiness::GetScene()
{
	if (UWorld* World = GetWorld())
	{
		if (FPhysScene* Scene = World->GetPhysicsScene())
		{
			return static_cast<FPhysScene_Chaos*>(Scene);
		}
	}
	return nullptr;
}

Chaos::FPhysicsSolver* USpatialReadiness::GetSolver()
{
	if (FPhysScene* Scene = GetScene())
	{
		if (Chaos::FPhysicsSolver* Solver = Scene->GetSolver())
		{
			return Solver;
		}
	}

	return nullptr;
}

