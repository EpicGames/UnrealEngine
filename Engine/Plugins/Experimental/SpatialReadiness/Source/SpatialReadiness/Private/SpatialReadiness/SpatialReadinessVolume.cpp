// Copyright Epic Games, Inc. All Rights Reserved.

#include "SpatialReadinessVolume.h"
#include "SpatialReadinessAPIDelegates.h"

FSpatialReadinessVolume::FSpatialReadinessVolume(
	TWeakPtr<FSpatialReadinessAPIDelegates> InDelegates,
	const FBox& InBounds,
	const FString& InDescription)
	: Delegates(InDelegates)
	, Index(INDEX_NONE)
	, Bounds(InBounds)
#if WITH_SPATIAL_READINESS_DESCRIPTIONS
	, Description(InDescription)
#endif
{ }

FSpatialReadinessVolume::FSpatialReadinessVolume(FSpatialReadinessVolume&& Other)
	: Delegates(Other.Delegates)
	, Index(Other.Index)
	, Bounds(Other.Bounds)
#if WITH_SPATIAL_READINESS_DESCRIPTIONS
	, Description(Other.Description)
#endif
{
	Other.Delegates.Reset();
	Other.Index = INDEX_NONE;
	Other.Bounds = FBox();
#if WITH_SPATIAL_READINESS_DESCRIPTIONS
	Other.Description = TEXT("");
#endif
}

FSpatialReadinessVolume& FSpatialReadinessVolume::operator=(FSpatialReadinessVolume&& Other)
{
	Delegates = Other.Delegates;
	Index = Other.Index;
	Bounds = Other.Bounds;
	Other.Delegates.Reset();
	Other.Index = INDEX_NONE;
	Other.Bounds = FBox();
#if WITH_SPATIAL_READINESS_DESCRIPTIONS
	Description = Other.Description;
	Other.Description = TEXT("");
#endif
	return *this;
}

FSpatialReadinessVolume::~FSpatialReadinessVolume()
{
	// When a handle is destroyed, we mark the corresponding volume as
	// "ready" (in case it wasn't already). This ensures that we do not
	// end up losing track of unready volumes and leaving them hanging
	// around in an acceleration structure.
	//
	// This is the only case where we actually do an IsValid check
	// without _ensuring_ validity. If the api which created this
	// handle has already been deleted, then MarkReady is not going to
	// do anything anyway...
	//
	// I'm a bit conflicted on adding this check because if we ever
	// come in here when IsValid == false, that means a handle was held
	// onto for too long.
	if (IsValid())
	{
		MarkReady();
	}
}

bool FSpatialReadinessVolume::IsValid() const
{
	return Delegates.IsValid();
}

bool FSpatialReadinessVolume::EnsureIsValid() const
{
	return ensureMsgf(IsValid(), TEXT("Readiness volume handle with invalid delegates pointer is being accessed"));
}

bool FSpatialReadinessVolume::IsReady() const
{
	EnsureIsValid();

	// If we have no index, that means there's no underlying unready volume,
	// so we're "ready"
	return Index == INDEX_NONE;
}

void FSpatialReadinessVolume::MarkReady()
{
	if (!IsValid())
	{
		return;
	}

	if (IsReady())
	{
		return;
	}

	TSharedPtr<FSpatialReadinessAPIDelegates> SharedDelegates = Delegates.Pin();
	if (ensureMsgf(SharedDelegates, TEXT("Tried to mark ready a readiness volume handle with an invalid delegates container")))
	{
		// Remove the unready volume corresponding with our current index
		SharedDelegates->RemoveUnreadyVolumeDelegate.ExecuteIfBound(Index);
	}

	// Stop tracking this index
	Index = INDEX_NONE;
}

void FSpatialReadinessVolume::MarkUnready()
{
	MarkUnready_Internal(false);
}

void FSpatialReadinessVolume::MarkUnready_Internal(const bool bRecreateState)
{
	// If we want to for sure recreate state, then just mark ready...
	// it's a compact way to ensure that we will need to regenerate
	// the unready volume.
	if (bRecreateState)
	{
		MarkReady();
	}

	if (!IsReady())
	{
		return;
	}

	TSharedPtr<FSpatialReadinessAPIDelegates> SharedDelegates = Delegates.Pin();
	if (ensureMsgf(SharedDelegates, TEXT("Tried to mark unready a readiness volume handle with an invalid delegates container")))
	{
		// If we've got an index, that means we're already associated with an
		// existing unready volume, which must be destroyed.
		if (Index != INDEX_NONE)
		{
			// If we already had an unread volume, remove it so we can make
			// a new one without leaving the old one dangling
			SharedDelegates->RemoveUnreadyVolumeDelegate.ExecuteIfBound(Index);
		}

		// Update our index to a new unready volume
		if (SharedDelegates->AddUnreadyVolumeDelegate.IsBound())
		{
			Index = SharedDelegates->AddUnreadyVolumeDelegate.Execute(Bounds, GetDescription());
		}
	}
}

void FSpatialReadinessVolume::MarkUnready(const FBox& NewBounds)
{
	const bool bRecreate = (Bounds != NewBounds);

	// Store new bounds
	Bounds = NewBounds;

	// Create a new unready volume with new bounds
	MarkUnready_Internal(bRecreate);
}

void FSpatialReadinessVolume::MarkUnready(const FString& NewDescription)
{
	const bool bRecreate = 
#if WITH_SPATIAL_READINESS_DESCRIPTIONS
		!Description.Equals(NewDescription);
	Description = NewDescription;
#else
		false;
#endif

	MarkUnready_Internal(bRecreate);
}

void FSpatialReadinessVolume::MarkUnready(const FBox& NewBounds, const FString& NewDescription)
{
	if (Bounds != NewBounds)
	{
		Bounds = NewBounds;
		MarkReady();
	}

#if WITH_SPATIAL_READINESS_DESCRIPTIONS
	if (!Description.Equals(NewDescription))
	{
		Description = NewDescription;
		MarkReady();
	}
#endif

	MarkUnready();
}

void FSpatialReadinessVolume::SetDescription(const FString& NewDescription)
{
#if WITH_SPATIAL_READINESS_DESCRIPTIONS
	if ((!IsValid()) || IsReady())
	{
		Description = NewDescription;
	}
	else
	{
		MarkUnready(NewDescription);
	}
#endif
}

void FSpatialReadinessVolume::SetBounds(const FBox& NewBounds)
{
	if ((!IsValid()) || IsReady())
	{
		Bounds = NewBounds;
	}
	else
	{
		MarkUnready(NewBounds);
	}
}

const FString& FSpatialReadinessVolume::GetDescription() const
{
#if WITH_SPATIAL_READINESS_DESCRIPTIONS
	return Description;
#else
	static const FString EmptyString;
	return EmptyString;
#endif
}

#if !UE_BUILD_SHIPPING 
const FString FSpatialReadinessVolume::GetDebugString() const
{
	return FString::Printf(TEXT("Spatial Readiness Volume [Description: %s][Bounds: %s : %s][Index: %d]"), *GetDescription(), *Bounds.Min.ToString(), *Bounds.Max.ToString(), Index);
}
#endif
