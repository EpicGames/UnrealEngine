// Copyright Epic Games, Inc. All Rights Reserved.

#include "SpatialReadinessVolumeComponent.h"
#include "SpatialReadinessSubsystem.h"
#include "Engine/World.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SpatialReadinessVolumeComponent)


void USpatialReadinessVolumeComponent::BeginPlay()
{
	Super::BeginPlay();

	if (UWorld* World = GetWorld())
	{
		if (USpatialReadiness* SpatialReadiness = World->GetSubsystem<USpatialReadiness>())
		{
			ReadinessVolume = SpatialReadiness->AddReadinessVolume(Bounds, Description);

			SetReadiness(bStartReady);
		}
	}
}

void USpatialReadinessVolumeComponent::EndPlay(const EEndPlayReason::Type EndPlayReason)
{
	Super::EndPlay(EndPlayReason);

	if (ReadinessVolume.IsSet())
	{
		ReadinessVolume->MarkReady();
	}
}

bool USpatialReadinessVolumeComponent::IsReady() const
{
	if (ReadinessVolume.IsSet())
	{
		return ReadinessVolume->IsReady();
	}

	return false;
}

void USpatialReadinessVolumeComponent::MarkReady()
{
	if (ReadinessVolume.IsSet())
	{
		ReadinessVolume->MarkReady();
	}
}

void USpatialReadinessVolumeComponent::MarkUnready()
{
	if (ReadinessVolume.IsSet())
	{
		ReadinessVolume->MarkUnready();
	}
}

void USpatialReadinessVolumeComponent::SetReadiness(bool bIsReady)
{
	if (bIsReady)
	{
		MarkReady();
	}
	else
	{
		MarkUnready();
	}
}

void USpatialReadinessVolumeComponent::SetDescription(const FString& InDescription)
{
	if (ReadinessVolume.IsSet())
	{
		ReadinessVolume->SetDescription(InDescription);
	}
}

void USpatialReadinessVolumeComponent::SetBounds(const FBox& InBounds)
{
	if (ReadinessVolume.IsSet())
	{
		ReadinessVolume->SetBounds(InBounds);
	}
}
