// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/Function.h"
#include "SpatialReadinessVolume.h"
#include "SpatialReadinessSignatures.h"

struct FSpatialReadinessAPIDelegates;

class FSpatialReadinessAPI final
{
public:

	// This constructor binds TFunctions to the internal add and remove delegates
	SPATIALREADINESS_API FSpatialReadinessAPI(
		FAddUnreadyVolume_Function AddUnreadyVolume,
		FRemoveUnreadyVolume_Function RemoveUnreadyVolume);

	// This constructor binds member functions to the internal add and remove delegates.
	//
	// Please be sure that this API object will not outlive the object to whose members
	// its delegates are bound after this construction!
	template <typename ObjectT>
	FSpatialReadinessAPI(ObjectT* Object,
		TAddUnreadyVolume_Member<ObjectT> AddUnreadyVolume,
		TRemoveUnreadyVolume_Member<ObjectT> RemoveUnreadyVolume);

	SPATIALREADINESS_API ~FSpatialReadinessAPI();

	SPATIALREADINESS_API FSpatialReadinessVolume CreateVolume(const FBox& Bounds, const FString& Description);

private:

	TSharedPtr<FSpatialReadinessAPIDelegates> Delegates;
};

template <typename ObjectT>
FSpatialReadinessAPI::FSpatialReadinessAPI(ObjectT* Object,
	TAddUnreadyVolume_Member<ObjectT> AddUnreadyVolume,
	TRemoveUnreadyVolume_Member<ObjectT> RemoveUnreadyVolume)
	: FSpatialReadinessAPI(
		[Object, AddUnreadyVolume] (const FBox& Bounds, const FString& Desc) -> int32
		{ return (Object->*AddUnreadyVolume)(Bounds, Desc); },
		[Object, RemoveUnreadyVolume] (int32 Index) -> void
		{ (Object->*RemoveUnreadyVolume)(Index); })
{ }