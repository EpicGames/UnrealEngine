// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#ifndef WITH_SPATIAL_READINESS_DESCRIPTIONS
#define WITH_SPATIAL_READINESS_DESCRIPTIONS (!(UE_BUILD_SHIPPING || UE_BUILD_TEST))
#endif

#ifndef WITH_SPATIAL_READINESS_CVD
#define WITH_SPATIAL_READINESS_CVD (!(UE_BUILD_SHIPPING || UE_BUILD_TEST))
#endif

