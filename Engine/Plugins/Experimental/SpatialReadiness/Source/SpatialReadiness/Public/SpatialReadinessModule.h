// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"

class FSpatialReadinessModule : public IModuleInterface
{
public:
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};