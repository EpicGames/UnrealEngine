// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "CoreMinimal.h"
#include "GenericPlatform/GenericPlatformMisc.h"
#include "SpatialReadinessSignatures.generated.h"

// Declare external delegate types
UENUM()
enum class EUnreadyVolumeAction : uint8 { Added, Removed };
DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnUnreadyVolumeChanged_Delegate, const FBox&, const FString&, EUnreadyVolumeAction);

// Declare internal delegate types
DECLARE_DELEGATE_RetVal_TwoParams(int32, FAddUnreadyVolume_Delegate, const FBox&, const FString&);
DECLARE_DELEGATE_OneParam(FRemoveUnreadyVolume_Delegate, int32);

// Declare TFunction types which can be bound to the delegates
using FAddUnreadyVolume_Function = TFunction<int32(const FBox&, const FString&)>;
using FRemoveUnreadyVolume_Function = TFunction<void(int32)>;

// Declare member function types which can be bound to delegates
template <typename ObjectT>
using TAddUnreadyVolume_Member = int32(ObjectT::*)(const FBox&, const FString&);
template <typename ObjectT>
using TRemoveUnreadyVolume_Member = void(ObjectT::*)(int32);
