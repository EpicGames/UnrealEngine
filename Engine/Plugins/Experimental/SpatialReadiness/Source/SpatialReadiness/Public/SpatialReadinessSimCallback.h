// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Chaos/SimCallbackObject.h"
#include "Chaos/PBDRigidsEvolutionFwd.h"
#include "Chaos/Particle/ObjectState.h"
#include "Chaos/Framework/HashMappedArray.h"
#include "SpatialReadinessVolume.h"

class FPhysScene_Chaos;
struct FProxyTimestampBase;
namespace Chaos
{
	class FSingleParticlePhysicsProxy;
	enum class EObjectStateType: int8;
}

enum ESpatialReadinessSimCallback_MidPhaseStrategy
{
	PrioritizeUnreadyVolumes = 0,
	PrioritizeMidPhases,
	Grid,
	COUNT
};


struct FUnreadyVolumeData_GT
{
	FUnreadyVolumeData_GT(Chaos::FSingleParticlePhysicsProxy* InProxy, const FBox& InBounds, const FString& InDescription)
		: Proxy(InProxy)
		, Bounds(InBounds)
#if WITH_SPATIAL_READINESS_DESCRIPTIONS
		, Description(InDescription)
#endif
	{ }
	Chaos::FSingleParticlePhysicsProxy* Proxy;
	FBox Bounds;
#if WITH_SPATIAL_READINESS_DESCRIPTIONS
	FString Description;
#endif
};

struct FSpatialReadinessSimCallbackInput : public Chaos::FSimCallbackInput
{
	TSet<Chaos::FSingleParticlePhysicsProxy*> UnreadyVolumesToAdd;
	TSet<Chaos::FSingleParticlePhysicsProxy*> UnreadyVolumesToRemove;

	void Reset()
	{
		UnreadyVolumesToAdd.Reset();
		UnreadyVolumesToRemove.Reset();
	}
};

struct FSpatialReadinessSimCallback
	: public Chaos::TSimCallbackObject<
		FSpatialReadinessSimCallbackInput,
		Chaos::FSimCallbackNoOutput,
		Chaos::ESimCallbackOptions::Presimulate |
		Chaos::ESimCallbackOptions::ParticleRegister |
		Chaos::ESimCallbackOptions::MidPhaseModification |
		Chaos::ESimCallbackOptions::PreIntegrate |
		Chaos::ESimCallbackOptions::PostIntegrate |
		Chaos::ESimCallbackOptions::PreSolve |
		Chaos::ESimCallbackOptions::PostSolve>
{
	using This = FSpatialReadinessSimCallback;

public:
	FSpatialReadinessSimCallback(FPhysScene_Chaos& InPhysicsScene);

	// Game thread functions for adding and removing unready volumes
	int32 AddUnreadyVolume_GT(const FBox& Bounds, const FString& Description);
	void RemoveUnreadyVolume_GT(int32 UnreadyVolumeIndex);

	// Game thread function for querying for unready volumes
	//
	// If bAllUnreadyVolumes is true, then a multi-query will be used and
	// OutVolumeIndices will be populated with every index that encroaches.
	//
	// For performance, we only return the index of the first unread volume
	// that we find.
	bool QueryReadiness_GT(const FBox& Bounds, TArray<int32>& OutVolumeIndices, bool bAllUnreadyVolumes=false) const;

	// Given a volume index, get it's description
	const FUnreadyVolumeData_GT* GetVolumeData_GT(int32 VolumeIndex) const;

	// Iterate over each unready volume and get 
	void ForEachVolumeData_GT(const TFunction<void(const FUnreadyVolumeData_GT&)>& Func);

	// Get the number of unready volumes
	int32 GetNumUnreadyVolumes_GT() const;

	// Perform an operation on each of the currently frozen particles - only valid on the physics thread
	void ForEachUnreadyRigidParticle_PT(const TFunction<bool(Chaos::FPBDRigidParticleHandle*)>& Lambda) const;

	// A variant which also sees how many frames each particle will remain frozen
	void ForEachUnreadyRigidParticle_PT(const TFunction<bool(Chaos::FPBDRigidParticleHandle*, int32)>& Lambda) const;

	// Get the total number of currently frozen particles - only valid on the physics thread
	int32 GetNumUnreadyRigidParticles_PT() const;

	/* begin: TSimCallbackObject */
protected:
	virtual void OnPreSimulate_Internal() override;
	virtual void OnParticlesRegistered_Internal(TArray<Chaos::FSingleParticlePhysicsProxy*>& RegisteredProxies) override;
	virtual void OnMidPhaseModification_Internal(Chaos::FMidPhaseModifierAccessor& Accessor) override;
	virtual void OnPreIntegrate_Internal() override;
	virtual void OnPostIntegrate_Internal() override;
	virtual void OnPreSolve_Internal() override;
	virtual void OnPostSolve_Internal() override;
	/* end: TSimCallbackObject */

	// Variants of the midphase modification function, with different
	// performance characteristics
	void OnMidPhaseModification_PrioritizeUnreadyVolumes(Chaos::FMidPhaseModifierAccessor& Accessor);
	void OnMidPhaseModification_PrioritizeMidPhases(Chaos::FMidPhaseModifierAccessor& Accessor);
	void OnMidPhaseModification_Grid(Chaos::FMidPhaseModifierAccessor& Accessor);

	// Helpers
	Chaos::FPBDRigidsEvolution* GetEvolution();
	static int32 GetFreezeFrames();
	ESpatialReadinessSimCallback_MidPhaseStrategy GetMidPhaseStrategy(const Chaos::FMidPhaseModifierAccessor& Accessor) const;

	// Functions for freezing and unfreezing all particles in the
	// UnreadyRigidParticles_PT set.
	void FreezeParticles_PT();
	void UnFreezeParticles_PT();

	// Decrement counters for each unready particle, removing them from the unready
	// particles array when their counters reach zero.
	void DecrementUnreadyRigidParticles_PT();

	// Physics thread function for querying for unready volumes
	// TODO: Make const
	bool QueryReadiness_PT(const Chaos::FAABB3& Bounds, TArray<const Chaos::FSingleParticlePhysicsProxy*>& OutVolumeProxies);

	// True for particles which are in the unready volumes particles list
	bool IsUnreadyVolume_PT(const Chaos::FGeometryParticleHandle& ParticleHandle) const;

	// Keep a ref to the phys scene so we can add and remove particles
	FPhysScene_Chaos& PhysicsScene;

	// List of unready volume physics proxies. We directly use single particle
	// physics proxy rather than something more generic because we know that
	// we are only going to create static single particles for these volumes. 
	struct FHashMapTraits
	{
		static uint32 GetIDHash(const int32 Idx) { return MurmurFinalize32(Idx); }
		static uint32 GetElementID(const FUnreadyVolumeData_GT& Element);
	};
	Chaos::Private::THashMappedArray<int32, FUnreadyVolumeData_GT, FHashMapTraits> UnreadyVolumeData_GT;

	// List of particle handles which represent unready volumes
	TSet<Chaos::FSingleParticlePhysicsProxy*> UnreadyVolumeParticles_PT;

	// Lists of particle handles which represent particles that interacted with unready
	// volumes, along with a counter representing how many more ticks the particle should
	// remain frozen.
	//
	// The counter strategy is to accommodate the following oddities about particle
	// creation, deletion, and midphase creation.
	//
	//  - A particle can exist for one entire sim tick after it is created without
	//    appearing in the mid phase. Therefore particles which are registered on top of
	//    unready volumes must be unreadied with a count of 2.
	//
	//  - For the same reason as the item above, when an unready volume is deleted
	//    on the same frame that another particle is created (perhaps a streaming object
	//    with which the unready volume was meant to prevent overlap), there's an
	//    additional frame during which the new particle has no collision. We lose
	//    nothing by keeping objects frozen for one more frame. The counter is used
	//    to accommadate this case.
	//
	// This freeze frame counter is packed into a small struct which also contains the
	// SyncTimeStamp for the particle's proxy, because if we're going to track the particle
	// for more than one frame (and we are), then we also need to track the validity of the
	// SyncTimeStamp before ever accessing methods or properties on the particle handle ptr.
	struct FUnreadyRigidParticleData
	{
		TWeakPtr<FProxyTimestampBase, ESPMode::ThreadSafe> SyncTimeStamp;
		int FreezeFrames;
	};
	TMap<Chaos::FPBDRigidParticleHandle*, FUnreadyRigidParticleData> UnreadyRigidParticles_PT;

	// "Unready" particles are forced to be stationary in PreSimulate, and restored
	// to their previous state in PostIntegrate. The values needed for restoration
	// are stored and mapped back to particle id's in this hash-mapped array.
	TArray<TPair<Chaos::FGeometryParticleHandle*, Chaos::EObjectStateType>> ParticleDataCache_PT;
};
