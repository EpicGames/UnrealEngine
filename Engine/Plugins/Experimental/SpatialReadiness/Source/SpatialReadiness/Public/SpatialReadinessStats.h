// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Stats/Stats.h"

DECLARE_STATS_GROUP(TEXT("SpatialReadiness"), STATGROUP_SpatialReadiness, STATCAT_Advanced);
DECLARE_STATS_GROUP_VERBOSE(TEXT("SpatialReadinessDetailed"), STATGROUP_SpatialReadinessVerbose, STATCAT_Advanced)

// Physics
DECLARE_CYCLE_STAT(TEXT("SpatialReadiness_Physics_AddUnreadyVolumeGT"), STAT_SpatialReadiness_Physics_AddUnreadyVolumeGT, STATGROUP_SpatialReadiness);
DECLARE_CYCLE_STAT(TEXT("SpatialReadiness_Physics_RemoveUnreadyVolumeGT"), STAT_SpatialReadiness_Physics_RemoveUnreadyVolumeGT, STATGROUP_SpatialReadiness);
DECLARE_CYCLE_STAT(TEXT("SpatialReadiness_Physics_FreezeParticlesPT"), STAT_SpatialReadiness_Physics_FreezeParticlesPT, STATGROUP_SpatialReadiness);
DECLARE_CYCLE_STAT(TEXT("SpatialReadiness_Physics_UnFreezeParticlesPT"), STAT_SpatialReadiness_Physics_UnFreezeParticlesPT, STATGROUP_SpatialReadiness);
DECLARE_CYCLE_STAT(TEXT("SpatialReadiness_Physics_DecrementUnreadyRigidParticlesPT"), STAT_SpatialReadiness_Physics_DecrementUnreadyRigidParticlesPT, STATGROUP_SpatialReadiness);
DECLARE_CYCLE_STAT(TEXT("SpatialReadiness_Physics_PreSimulate"), STAT_SpatialReadiness_Physics_PreSimulate, STATGROUP_SpatialReadiness);
DECLARE_CYCLE_STAT(TEXT("SpatialReadiness_Physics_ParticlesRegistered"), STAT_SpatialReadiness_Physics_ParticlesRegistered, STATGROUP_SpatialReadiness);
DECLARE_CYCLE_STAT(TEXT("SpatialReadiness_Physics_MidPhase"), STAT_SpatialReadiness_Physics_MidPhase, STATGROUP_SpatialReadiness);
DECLARE_CYCLE_STAT(TEXT("SpatialReadiness_Physics_MidPhase_PrioritizeUnreadyVolumes"), STAT_SpatialReadiness_Physics_MidPhase_PrioritizeUnreadyVolumes, STATGROUP_SpatialReadiness);
DECLARE_CYCLE_STAT(TEXT("SpatialReadiness_Physics_MidPhase_PrioritizeMidPhases"), STAT_SpatialReadiness_Physics_MidPhase_PrioritizeMidPhases, STATGROUP_SpatialReadiness);
DECLARE_CYCLE_STAT(TEXT("SpatialReadiness_Physics_MidPhase_Grid"), STAT_SpatialReadiness_Physics_MidPhase_Grid, STATGROUP_SpatialReadiness);
//DECLARE_CYCLE_STAT(TEXT("SpatialReadiness_Physics_MidPhase_AccessorLoop"), STAT_SpatialReadiness_Physics_MidPhase_AccessorLoop, STATGROUP_SpatialReadinessVerbose);
DECLARE_CYCLE_STAT(TEXT("SpatialReadiness_Physics_ChaosVisualDebugger"), STAT_SpatialReadiness_Physics_ChaosVisualDebugger, STATGROUP_SpatialReadiness);
DECLARE_CYCLE_STAT(TEXT("SpatialReadiness_Physics_DebugDraw"), STAT_SpatialReadiness_Physics_DebugDraw, STATGROUP_SpatialReadiness);

