// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Chaos/Declares.h"
#include "Subsystems/WorldSubsystem.h"
#include "SpatialReadinessAPI.h"
#include "SpatialReadinessSimCallback.h"
#include "SpatialReadinessSignatures.h"
#include "SpatialReadinessSubsystem.generated.h"

class APlayerController;
class UCanvas;

UCLASS(MinimalAPI)
class USpatialReadiness : public UWorldSubsystem
{
	GENERATED_BODY()
	using This = USpatialReadiness;

public:
	USpatialReadiness();
	USpatialReadiness(FVTableHelper& Helper);

	// Add a volume which can be marked read/unready
	SPATIALREADINESS_API FSpatialReadinessVolume AddReadinessVolume(const FBox& Bounds, const FString& Description);

	// Check to see if a volume is ready - returns true if ready.
	//
	// If bAllDescriptions = false, OutDescriptions will contain only the description of the
	// first unready volume that we hit. If true, it will contain descriptions from all
	// overlapping volumes.
	SPATIALREADINESS_API bool QueryReadiness(const FBox& Bounds, TArray<FString>& OutDescriptions, bool bAllDescriptions = false) const;

	// Get the number of unready volumes
	SPATIALREADINESS_API uint32 GetNumUnreadyVolumes() const;

	// Add a function of some type to the external-facing delegate for unready volume updates
	template <typename ReceiverT>
	FDelegateHandle OnUnreadyVolumeChangedDelegate_AddUObject(ReceiverT* Receiver, void (ReceiverT::*Method)(const FBox&, const FString&, EUnreadyVolumeAction))
	{ return OnUnreadyVolumeChanged_Delegate.AddUObject(Receiver, Method); }

	// Remove a function from the external-facing delegate for unready volume updates
	SPATIALREADINESS_API void OnUnreadyVolumeChangedDelegate_Remove(const FDelegateHandle& Handle);

#if !UE_BUILD_SHIPPING
	FSpatialReadinessSimCallback* GetSimCallback() { return SimCallback; }
	const FSpatialReadinessSimCallback* GetSimCallback() const { return SimCallback; }
#endif

#if WITH_EDITOR
	FDelegateHandle OnDebugDrawDelegateHandle;
	void OnDebugDraw(UCanvas* Canvas, APlayerController* PlayerController);
#endif

	SPATIALREADINESS_API static bool StaticShouldCreateSubsystem(UObject* Outer);

	/* begin: UWorldSubsystem */
protected:
	virtual bool ShouldCreateSubsystem(UObject* Outer) const override;
	virtual void OnWorldBeginPlay(UWorld& InWorld) override;
	virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	virtual void Deinitialize() override;
	/* end: UWorldSubsystem */

private:

	// The spatial readiness API object which manages giving out volume "handles"
	FSpatialReadinessAPI SpatialReadiness;

	// The functions to bind to the spatial readiness API.
	int32 AddUnreadyVolume(const FBox& Bounds, const FString& Description);
	void RemoveUnreadyVolume(int32 UnreadyVolumeIndex);

	// Create and destroy the physics sim callback
	bool CreateSimCallback();
	bool DestroySimCallback();

	// Getters
	FPhysScene_Chaos* GetScene();
	Chaos::FPhysicsSolver* GetSolver();

	FOnUnreadyVolumeChanged_Delegate OnUnreadyVolumeChanged_Delegate;

	FSpatialReadinessSimCallback* SimCallback = nullptr;
};
