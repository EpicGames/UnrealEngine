// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Math/MathFwd.h"
#include "SpatialReadinessDefines.h"

struct FSpatialReadinessAPIDelegates;

class FSpatialReadinessVolume final
{
public:

	// Check to make sure that the readiness system which created this handle
	// did not go out of scope.
	SPATIALREADINESS_API bool IsValid() const;

	// Ensure that IsValid() returns true, printing a message and returning
	// false if not.
	SPATIALREADINESS_API bool EnsureIsValid() const;

	// Check to see if this handle points to a volume which is currently "ready".
	SPATIALREADINESS_API bool IsReady() const;

	// Mark this volume as "ready"
	SPATIALREADINESS_API void MarkReady();

	// Mark this volume as "unready"
	SPATIALREADINESS_API void MarkUnready();

	// Optional overrides to MakeUnready() for adjusting bounds and/or description
	SPATIALREADINESS_API void MarkUnready(const FBox& NewBounds);
	SPATIALREADINESS_API void MarkUnready(const FString& NewDescription);
	SPATIALREADINESS_API void MarkUnready(const FBox& NewBounds, const FString& NewDescription);

	// Set properties, updating internals if necessary
	SPATIALREADINESS_API void SetDescription(const FString& NewDescription);
	SPATIALREADINESS_API void SetBounds(const FBox& NewBounds);

	// Get the description of this handle - will be empty string if
	// the WITH_SPATIAL_READINESS_DESCRIPTIONS macro is 0
	SPATIALREADINESS_API const FString& GetDescription() const;

#if !UE_BUILD_SHIPPING 
	// Get a debug string for this volume
	SPATIALREADINESS_API const FString GetDebugString() const;
#endif

	// When a handle is destroyed, it will make the corresponding volume as ready
	SPATIALREADINESS_API ~FSpatialReadinessVolume();

	// Readiness handles are unique and cannot be copied
	FSpatialReadinessVolume(const FSpatialReadinessVolume&) = delete;
	FSpatialReadinessVolume& operator=(const FSpatialReadinessVolume&) = delete;

	// Move constructor & assignment
	SPATIALREADINESS_API FSpatialReadinessVolume(FSpatialReadinessVolume&& Other);
	SPATIALREADINESS_API FSpatialReadinessVolume& operator=(FSpatialReadinessVolume&& Other);

private:

	void MarkUnready_Internal(bool bRecreateState);

	// We keep the constructor private and befriend the API object to restrict
	// the means by which a handle can be created.
	friend class FSpatialReadinessAPI;
	FSpatialReadinessVolume(
		TWeakPtr<FSpatialReadinessAPIDelegates> InDelegates,
		const FBox& InBounds,
		const FString& InDescription);

	// Ref to the spatial readiness system. It will be an access error to use
	// this handle if this interface ever goes out of scope
	TWeakPtr<FSpatialReadinessAPIDelegates> Delegates;

	// Index used by the interface to select the related volume, if
	// one exists (ie if this handle refers to an "unready" state). If one
	// doesn't exist, then this handle is "ready" and this is INDEX_NONE
	int32 Index;

	// Bounds of the unready volume. This is used when marking the volume
	// as unready to create the internal "unready volume".
	FBox Bounds;

#if WITH_SPATIAL_READINESS_DESCRIPTIONS
	// Description of the reason why the volume referred to by this handle
	// might be unready. Again, this is used only when creating the internal
	// unready volume.
	FString Description;
#endif
};
