// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/ActorComponent.h"
#include "SpatialReadinessVolume.h"
#include "SpatialReadinessVolumeComponent.generated.h"

UCLASS(Blueprintable, meta = (BlueprintSpawnableComponent), MinimalAPI)
class USpatialReadinessVolumeComponent : public UActorComponent
{
	GENERATED_BODY()

public:

	virtual void BeginPlay() override;
	virtual void EndPlay(const EEndPlayReason::Type EndPlayReason) override;

	UFUNCTION(BlueprintCallable, Category = "SpatialReadiness")
	bool IsReady() const;

	UFUNCTION(BlueprintCallable, Category = "SpatialReadiness")
	void MarkReady();

	UFUNCTION(BlueprintCallable, Category = "SpatialReadiness")
	void MarkUnready();

	UFUNCTION(BlueprintCallable, Category = "SpatialReadiness")
	void SetReadiness(bool bIsReady);

	UFUNCTION(BlueprintCallable, Category = "SpatialReadiness")
	void SetDescription(const FString& InDescription);

	UFUNCTION(BlueprintCallable, Category = "SpatialReadiness")
	void SetBounds(const FBox& InBounds);

	UPROPERTY(EditDefaultsOnly, Category = "SpatialReadiness", BlueprintSetter=SetDescription)
	FString Description = "SpatialReadinessVolumeComponent";

	UPROPERTY(EditDefaultsOnly, Category = "SpatialReadiness", BlueprintSetter=SetBounds)
	FBox Bounds = FBox(FVector(-50.f, -50.f, -50.f), FVector(50.f, 50.f, 50.f));

	UPROPERTY(EditDefaultsOnly, Category = "SpatialReadiness")
	bool bStartReady = false;

protected:

	TOptional<FSpatialReadinessVolume> ReadinessVolume;
};

