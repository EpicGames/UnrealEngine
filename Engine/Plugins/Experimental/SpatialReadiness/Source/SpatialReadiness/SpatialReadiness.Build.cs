// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class SpatialReadiness : ModuleRules
{
    public SpatialReadiness(ReadOnlyTargetRules Target) : base(Target)
    {
        PrivateDependencyModuleNames.AddRange(
            new string[]
            {
                "Engine",
				"CoreUObject",
            });

        PublicDependencyModuleNames.AddRange(
            new string[] {
                "Core",
            }
        );

		SetupModulePhysicsSupport(Target);
    }
}
