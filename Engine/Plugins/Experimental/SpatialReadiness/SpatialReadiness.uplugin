{
	"FileVersion": 1,
	"CanContainContent": false,
	"ExplicitlyLoaded": false,
	"EnabledByDefault": false,
	"Modules": [
		{
			"Name": "SpatialReadiness",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "ChaosUserDataPT",
			"Enabled": true
		}
	]
}
